// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class HardwareTargeting : ModuleRules
{
	public HardwareTargeting(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"Engine",
				"InputCore",
				"SlateCore",
				"Slate",
				"EditorFramework",
				"UnrealEd",
				"EditorWidgets",
				"Settings",
				"EngineSettings",
			}
		);
	}
}
