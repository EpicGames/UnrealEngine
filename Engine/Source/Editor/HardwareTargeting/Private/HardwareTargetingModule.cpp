// Copyright Epic Games, Inc. All Rights Reserved.

#include "HardwareTargetingModule.h"

#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Engine/RendererSettings.h"
#include "GameFramework/InputSettings.h"
#include "GameMapsSettings.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IDocumentation.h"
#include "ISettingsModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Modules/ModuleManager.h"
#include "SEnumCombo.h"
#include "SceneUtils.h"
#include "Settings/EditorProjectSettings.h"
#include "SlateSettings.h"
#include "Templates/Tuple.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "HardwareTargeting"


//////////////////////////////////////////////////////////////////////////
// FMetaSettingGatherer

struct FMetaSettingGatherer
{
	FTextBuilder DescriptionBuffer;

	TMap<UObject*, FTextBuilder> DescriptionBuffers;
	TMap<UObject*, FText> CategoryNames;

	// Are we just displaying what would change, or actually changing things?
	bool bReadOnly;

	bool bIncludeUnmodifiedProperties;

	FMetaSettingGatherer()
		: bReadOnly(false)
		, bIncludeUnmodifiedProperties(false)
	{
	}

	void AddEntry(UObject* SettingsObject, FProperty* Property, FText NewValue, bool bModified)
	{
		if (bModified || bIncludeUnmodifiedProperties)
		{
			FTextBuilder& SettingsDescriptionBuffer = DescriptionBuffers.FindOrAdd(SettingsObject);

			if (!bReadOnly)
			{
				FPropertyChangedEvent ChangeEvent(Property, EPropertyChangeType::ValueSet);
				SettingsObject->PostEditChangeProperty(ChangeEvent);
			}
			else
			{
				FText SettingDisplayName = Property->GetDisplayNameText();

				FFormatNamedArguments Args;
				Args.Add(TEXT("SettingName"), SettingDisplayName);
				Args.Add(TEXT("SettingValue"), NewValue);

				FText FormatString = bModified ?
					LOCTEXT("MetaSettingDisplayStringModified", "{SettingName} is {SettingValue} <HardwareTargets.Strong>(modified)</>") :
					LOCTEXT("MetaSettingDisplayStringUnmodified", "{SettingName} is {SettingValue}");

				SettingsDescriptionBuffer.AppendLine(FText::Format(FormatString, Args));
			}
		}
	}

	template <typename ValueType>
	static FText ValueToString(ValueType Value);

	bool Finalize()
	{
		check(!bReadOnly);

		bool bSuccess = true;
		for (auto& Pair : DescriptionBuffers)
		{
			const FString Filename = Pair.Key->GetDefaultConfigFilename();
			const FDateTime BeforeTime = IFileManager::Get().GetTimeStamp(*Filename);

			Pair.Key->TryUpdateDefaultConfigFile();

			const FDateTime AfterTime = IFileManager::Get().GetTimeStamp(*Filename);
			bSuccess = BeforeTime != AfterTime && bSuccess;
		}

		return bSuccess;
	}
};


template <>
FText FMetaSettingGatherer::ValueToString(bool Value)
{
	return Value ? LOCTEXT("BoolEnabled", "enabled") : LOCTEXT("BoolDisabled", "disabled");
}

template <>
FText FMetaSettingGatherer::ValueToString(EAntiAliasingMethod Value)
{
	switch (Value)
	{
	case AAM_None:
		return LOCTEXT("AA_None", "None");
	case AAM_FXAA:
		return LOCTEXT("AA_FXAA", "Fast Approximate Anti-Aliasing (FXAA)");
	case AAM_TemporalAA:
		return LOCTEXT("AA_TemporalAA", "Temporal Anti-Aliasing (TAA)");
	case AAM_MSAA:
		return LOCTEXT("AA_MSAA", "Multisample Anti-Aliasing (MSAA)");
	case AAM_TSR:
		return LOCTEXT("AAM_TSR", "Temporal Super-Resolution (TSR)");
	case AAM_SMAA:
		return LOCTEXT("AA_SMAA", "Subpixel Morphological Anti-Aliasing (SMAA)");
	default:
		return FText::AsNumber((int32)Value);
	}
}

static FName HardwareTargetingConsoleVariableMetaFName(TEXT("ConsoleVariable"));

#define UE_META_SETTING_ENTRY(Builder, Class, PropertyName, TargetValue) \
{ \
	Class* SettingsObject = GetMutableDefault<Class>(); \
	bool bModified = SettingsObject->PropertyName != (TargetValue); \
	FProperty* Property = FindFieldChecked<FProperty>(Class::StaticClass(), GET_MEMBER_NAME_CHECKED(Class, PropertyName)); \
	if (!Builder.bReadOnly) { \
		const FString& CVarName = Property->GetMetaData(HardwareTargetingConsoleVariableMetaFName); \
		if (!CVarName.IsEmpty()) { IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName); \
			if (CVar) { CVar->Set(TargetValue, ECVF_SetByProjectSetting); } } \
		SettingsObject->PropertyName = (TargetValue); } \
	Builder.AddEntry(SettingsObject, Property, FMetaSettingGatherer::ValueToString(TargetValue), bModified); \
}

//////////////////////////////////////////////////////////////////////////
// FHardwareTargetingModule

class FHardwareTargetingModule : public IHardwareTargetingModule
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface interface
	
	// IHardwareTargetingModule interface
	virtual void ApplyHardwareTargetingSettings() override;
	virtual TArray<FModifiedDefaultConfig> GetPendingSettingsChanges() override;
	virtual TSharedRef<SWidget> MakeHardwareClassTargetCombo(FOnHardwareClassChanged OnChanged, TAttribute<EHardwareClass> SelectedEnum) override;
	virtual TSharedRef<SWidget> MakeGraphicsPresetTargetCombo(FOnGraphicsPresetChanged OnChanged, TAttribute<EGraphicsPreset> SelectedEnum) override;
	// End of IHardwareTargetingModule interface

private:
	void GatherSettings(FMetaSettingGatherer& Builder);
};

void FHardwareTargetingModule::StartupModule()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		
	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Project", "Project", "HardwareTargeting",
			LOCTEXT("HardwareTargetingSettingsName", "Target Hardware"),
			LOCTEXT("HardwareTargetingSettingsDescription", "Options for choosing which class of hardware to target"),
			GetMutableDefault<UHardwareTargetingSettings>()
		);
	}

	// Apply any settings on startup if necessary
	ApplyHardwareTargetingSettings();
}

void FHardwareTargetingModule::ShutdownModule()
{
}

TArray<FModifiedDefaultConfig> FHardwareTargetingModule::GetPendingSettingsChanges()
{
	// Gather and stringify the modified settings
	FMetaSettingGatherer Gatherer;
	Gatherer.bReadOnly = true;
	Gatherer.bIncludeUnmodifiedProperties = true;
	GatherSettings(Gatherer);

	TArray<FModifiedDefaultConfig> OutArray;
	for (auto& Pair : Gatherer.DescriptionBuffers)
	{
		FModifiedDefaultConfig ModifiedConfig;
		ModifiedConfig.SettingsObject = Pair.Key;
		ModifiedConfig.Description = Pair.Value.ToText();
		ModifiedConfig.CategoryHeading = Gatherer.CategoryNames.FindChecked(Pair.Key);

		OutArray.Add(ModifiedConfig);
	}
	return OutArray;
}
	
void FHardwareTargetingModule::GatherSettings(FMetaSettingGatherer& Builder)
{
	UHardwareTargetingSettings* Settings = GetMutableDefault<UHardwareTargetingSettings>();


	if (Builder.bReadOnly)
	{
		// Force the category order and give nice descriptions
		Builder.CategoryNames.Add(GetMutableDefault<URendererSettings>(), LOCTEXT("RenderingCategoryHeader", "Engine - Rendering"));
		Builder.CategoryNames.Add(GetMutableDefault<UInputSettings>(), LOCTEXT("InputCategoryHeader", "Engine - Input"));
		Builder.CategoryNames.Add(GetMutableDefault<UGameMapsSettings>(), LOCTEXT("MapsAndModesCategoryHeader", "Project - Maps & Modes"));
		Builder.CategoryNames.Add(GetMutableDefault<ULevelEditor2DSettings>(), LOCTEXT("EditorSettings2D", "Editor - 2D"));
		Builder.CategoryNames.Add(GetMutableDefault<USlateSettings>(), LOCTEXT("SlateCategoryHeader", "Slate"));
	}


	const bool bLowEndMobile = (Settings->TargetedHardwareClass == EHardwareClass::Mobile) && (Settings->DefaultGraphicsPerformance == EGraphicsPreset::Scalable);
	const bool bAnyMobile = (Settings->TargetedHardwareClass == EHardwareClass::Mobile);
	const bool bHighEndMobile = (Settings->TargetedHardwareClass == EHardwareClass::Mobile) && (Settings->DefaultGraphicsPerformance == EGraphicsPreset::Maximum);
	const bool bAnyPC = (Settings->TargetedHardwareClass == EHardwareClass::Desktop);
	const bool bHighEndPC = (Settings->TargetedHardwareClass == EHardwareClass::Desktop) && (Settings->DefaultGraphicsPerformance == EGraphicsPreset::Maximum);
	const bool bAnyScalable = Settings->DefaultGraphicsPerformance == EGraphicsPreset::Scalable;

	{
		// Bloom works and isn't terribly expensive on anything beyond low-end
		UE_META_SETTING_ENTRY(Builder, URendererSettings, bDefaultFeatureBloom, !bLowEndMobile);

		// Separate translucency
		UE_META_SETTING_ENTRY(Builder, URendererSettings, bSeparateTranslucency, !bAnyMobile);

		// Motion blur, auto-exposure, and ambient occlusion
		UE_META_SETTING_ENTRY(Builder, URendererSettings, bDefaultFeatureMotionBlur, bHighEndPC);
		UE_META_SETTING_ENTRY(Builder, URendererSettings, bDefaultFeatureAutoExposure, bHighEndPC);
		UE_META_SETTING_ENTRY(Builder, URendererSettings, bDefaultFeatureAmbientOcclusion, bAnyPC);

		// lens flare
		UE_META_SETTING_ENTRY(Builder, URendererSettings, bDefaultFeatureLensFlare, false);

		// DOF and AA work on mobile but are expensive, keeping them off by default
		//@TODO: DOF setting doesn't exist yet
		// UE_META_SETTING_ENTRY(Builder, URendererSettings, bDefaultFeatureDepthOfField, bHighEndPC);
		UE_META_SETTING_ENTRY(Builder, URendererSettings, DefaultFeatureAntiAliasing, bHighEndPC ? AAM_TSR : AAM_None);
	}

	{
		// Tablets or phones are usually shared-screen multiplayer instead of split-screen
		UE_META_SETTING_ENTRY(Builder, UGameMapsSettings, bUseSplitscreen, bAnyPC);
	}

	{
		// Enable explicit ZOrder for UMG canvas on mobile platform to improve batching
		UE_META_SETTING_ENTRY(Builder, USlateSettings, bExplicitCanvasChildZOrder, bAnyMobile);
	}
}

void FHardwareTargetingModule::ApplyHardwareTargetingSettings()
{
	UHardwareTargetingSettings* Settings = GetMutableDefault<UHardwareTargetingSettings>();

	// Apply the settings if they've changed
	if (Settings->HasPendingChanges())
	{
		// Gather and apply the modified settings
		FMetaSettingGatherer Builder;
		Builder.bReadOnly = false;
		GatherSettings(Builder);

		const bool bSuccess = Builder.Finalize();

		// Write out the 'did we apply' values
		if (bSuccess)
		{
			Settings->AppliedTargetedHardwareClass = Settings->TargetedHardwareClass;
			Settings->AppliedDefaultGraphicsPerformance = Settings->DefaultGraphicsPerformance;
			Settings->TryUpdateDefaultConfigFile();
		}
	}
}

TSharedRef<SWidget> FHardwareTargetingModule::MakeHardwareClassTargetCombo(FOnHardwareClassChanged OnChanged, TAttribute<EHardwareClass> SelectedEnum)
{
	const UEnum* HardwareClassEnum = StaticEnum<EHardwareClass>();

	return
		SNew(SEnumComboBox, HardwareClassEnum)
		.ContentPadding(FMargin(4.0f, 0.0f))
		.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("HardwareClassTooltip", "Choose the overall class of hardware to target (desktop/console or mobile/tablet)."), NULL, TEXT("Shared/Editor/Settings/TargetHardware"), TEXT("HardwareClass")))
		.OnEnumSelectionChanged_Lambda([OnChanged](int32 NewSelection, ESelectInfo::Type) { OnChanged.ExecuteIfBound(static_cast<EHardwareClass>(NewSelection)); })
		.CurrentValue_Lambda([SelectedEnum]() { return static_cast<int32>(SelectedEnum.Get()); });


		
}

TSharedRef<SWidget> FHardwareTargetingModule::MakeGraphicsPresetTargetCombo(FOnGraphicsPresetChanged OnChanged, TAttribute<EGraphicsPreset> SelectedEnum)
{
	const UEnum* GraphicsPresetEnum = StaticEnum<EGraphicsPreset>();
	
	return
		SNew(SEnumComboBox, GraphicsPresetEnum)
		.ContentPadding(FMargin(4.0f, 0.0f))
		.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("GraphicsPresetTooltip", "Choose the graphical level to target (high-end only or scalable from low-end on up)."), NULL, TEXT("Shared/Editor/Settings/TargetHardware"), TEXT("GraphicalLevel")))
		.OnEnumSelectionChanged_Lambda([OnChanged](int32 NewSelection, ESelectInfo::Type) { OnChanged.ExecuteIfBound(static_cast<EGraphicsPreset>(NewSelection)); })
		.CurrentValue_Lambda([SelectedEnum]() { return static_cast<int32>(SelectedEnum.Get()); });
}

IHardwareTargetingModule& IHardwareTargetingModule::Get()
{
	static FHardwareTargetingModule Instance;
	return Instance;
}

IMPLEMENT_MODULE(FHardwareTargetingModule, HardwareTargeting);

#undef UE_META_SETTING_ENTRY
#undef LOCTEXT_NAMESPACE
