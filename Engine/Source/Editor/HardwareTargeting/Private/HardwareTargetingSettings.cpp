// Copyright Epic Games, Inc. All Rights Reserved.

#include "HardwareTargetingSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HardwareTargetingSettings)


UHardwareTargetingSettings::UHardwareTargetingSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TargetedHardwareClass(EHardwareClass::Unspecified)
	, AppliedTargetedHardwareClass(EHardwareClass::Unspecified)
	, DefaultGraphicsPerformance(EGraphicsPreset::Unspecified)
	, AppliedDefaultGraphicsPerformance(EGraphicsPreset::Unspecified)
{ }


bool UHardwareTargetingSettings::HasPendingChanges() const
{
	if (TargetedHardwareClass == EHardwareClass::Unspecified || DefaultGraphicsPerformance == EGraphicsPreset::Unspecified)
	{
		return false;
	}

	return AppliedTargetedHardwareClass != TargetedHardwareClass || AppliedDefaultGraphicsPerformance != DefaultGraphicsPerformance;
}


void UHardwareTargetingSettings::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent )
{
	SettingChangedEvent.Broadcast();
}
