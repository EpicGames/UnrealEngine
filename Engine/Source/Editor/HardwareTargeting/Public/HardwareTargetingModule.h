// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HardwareTargetingSettings.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SWidget;
class UObject;

DECLARE_DELEGATE_OneParam(FOnHardwareClassChanged, EHardwareClass)
DECLARE_DELEGATE_OneParam(FOnGraphicsPresetChanged, EGraphicsPreset)

/** Struct specifying pending changes to a settings object */
struct FModifiedDefaultConfig
{
	/** The settings object to which the description relates */
	TWeakObjectPtr<UObject> SettingsObject;

	/** Heading describing the name of the category */
	FText CategoryHeading;

	/** Text describing the pending changes to the settings */
	FText Description;
};

class IHardwareTargetingModule : public IModuleInterface
{
public:

	/** Singleton access to this module */
	HARDWARETARGETING_API static IHardwareTargetingModule& Get();

	/** Apply the current hardware targeting settings if they have changed */
	virtual void ApplyHardwareTargetingSettings() = 0;

	/** Gets a list of objects that are required to be writable in order to apply the settings */
	virtual TArray<FModifiedDefaultConfig> GetPendingSettingsChanges() = 0;

	/** Make a new combo box for choosing a hardware class target */
	virtual TSharedRef<SWidget> MakeHardwareClassTargetCombo(FOnHardwareClassChanged OnChanged, TAttribute<EHardwareClass> SelectedEnum) = 0;

	/** Make a new combo box for choosing a graphics preference */
	virtual TSharedRef<SWidget> MakeGraphicsPresetTargetCombo(FOnGraphicsPresetChanged OnChanged, TAttribute<EGraphicsPreset> SelectedEnum) = 0;
};

