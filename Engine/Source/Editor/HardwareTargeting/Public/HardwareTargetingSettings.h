// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "HardwareTargetingSettings.generated.h"

#define UE_API HARDWARETARGETING_API

/** Enum specifying a class of hardware */
UENUM()
enum class EHardwareClass : uint8
{
	/** Unspecified, meaning no choice has been made yet */
	Unspecified UMETA(Hidden),

	/** Desktop or console */
	Desktop,

	/** Mobile or tablet */
	Mobile
};

/** Enum specifying a graphics preset preference */
UENUM()
enum class EGraphicsPreset : uint8
{
	/** Unspecified, meaning no choice has been made yet */
	Unspecified UMETA(Hidden),

	/** Maximum quality - High-end features default to enabled */
	Maximum,

	/** Scalable quality - Some features are disabled by default but can be enabled based on the actual hardware */
	Scalable
};

/** Hardware targeting settings, stored in default config, per-project */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UHardwareTargetingSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Enum specifying which class of hardware this game is targeting */
	UPROPERTY(config, EditAnywhere, category=None)
	EHardwareClass TargetedHardwareClass;
	
	/** Enum that is set to TargetedHardwareClass when the settings have been successfully applied */
	UPROPERTY(config)
	EHardwareClass AppliedTargetedHardwareClass;

	/** Enum specifying a graphics preset to use for this game */
	UPROPERTY(config, EditAnywhere, category=None)
	EGraphicsPreset DefaultGraphicsPerformance;

	/** Enum that is set to DefaultGraphicsPerformance when the settings have been successfully applied */
	UPROPERTY(config)
	EGraphicsPreset AppliedDefaultGraphicsPerformance;

	/** Check if these settings have any pending changes that require action */
	UE_API bool HasPendingChanges() const;

#if WITH_EDITOR
public:
	/** Returns an event delegate that is executed when a setting has changed. */
	DECLARE_EVENT(UHardwareTargetingSettings, FSettingChangedEvent);
	FSettingChangedEvent& OnSettingChanged( ) { return SettingChangedEvent; }

protected:
	/** Called when a property on this object is changed */
	UE_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent ) override;

private:
	/** Holds an event delegate that is executed when a setting has changed. */
	FSettingChangedEvent SettingChangedEvent;
#endif
};

#undef UE_API
